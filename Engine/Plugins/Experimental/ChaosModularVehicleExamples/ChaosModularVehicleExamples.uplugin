{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Chaos Modular Vehicle Examples",
	"Description": "Modular Vehicle Example Assets",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "Mover",
			"Enabled": true
		},
		{
			"Name": "ChaosMover",
			"Enabled": true
		}
	]
}
